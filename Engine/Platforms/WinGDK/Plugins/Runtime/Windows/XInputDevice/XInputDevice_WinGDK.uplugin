{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": [
		"WinGDK"
	],
	"Modules": [
		{
			"Name": "XInputDevice",
			"Type": "Runtime",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}