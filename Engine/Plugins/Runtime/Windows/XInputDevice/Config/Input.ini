[InputPlatformSettings_Windows InputPlatformSettings]
+HardwareDevices=(InputClassName="XInputInterface",HardwareDeviceIdentifier="XInputController",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=260)