// Copyright Epic Games, Inc. All Rights Reserved.

#include "XInputDeviceModule.h"
#include "XInputInterface.h"
#include "Modules/ModuleManager.h"	// For IMPLEMENT_MODULE

TSharedPtr<IInputDevice> FXInputDeviceModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	return XInputInterface::Create(InMessageHandler, true /* bShouldBePrimaryDevice */);
}

TSharedPtr<IInputDevice> FXInputDeviceModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, FInputDeviceCreationParameters InParameters)
{
	// XInputDevice can run on any thread and supports secondary device creation.
	return XInputInterface::Create(InMessageHandler, InParameters.bInitAsPrimaryDevice);
}

const TCHAR* FXInputDeviceModule::GetPreferredDeviceAPIString() const
{
	static const TCHAR* APIName = TEXT("XInput"); 
	return APIName;
}

IMPLEMENT_MODULE(FXInputDeviceModule, XInputDevice)
