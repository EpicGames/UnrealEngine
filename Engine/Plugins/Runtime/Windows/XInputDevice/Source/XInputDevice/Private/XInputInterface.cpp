// Copyright Epic Games, Inc. All Rights Reserved.

#include "XInputInterface.h"
#include "HAL/PlatformTime.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreMiscDefines.h"
#include "Windows/WindowsApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/InputDeviceRegistry.h"
#include "Misc/ConfigCacheIni.h"

#pragma pack (push,8)
#include "Windows/AllowWindowsPlatformTypes.h"
#include <XInput.h>
#include "Windows/HideWindowsPlatformTypes.h"
#pragma pack (pop)

#include "GameFramework/InputDeviceSubsystem.h"

static int32 ForceControllerStateUpdate = 0;
FAutoConsoleVariableRef CVarForceControllerStateUpdate(
	TEXT("XInput.ForceControllerStateUpdate"),
	ForceControllerStateUpdate,
	TEXT("Force XInput refresh of controller state on each frame.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

DEFINE_LOG_CATEGORY_STATIC(LogXInput, Log, All);

TSharedRef< XInputInterface > XInputInterface::Create(  const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler, bool bShouldBePrimaryDevice )
{
	return MakeShareable( new XInputInterface( InMessageHandler, bShouldBePrimaryDevice ) );
}


XInputInterface::XInputInterface(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, bool bShouldBePrimaryDevice)
	: bIsPrimaryDevice(bShouldBePrimaryDevice), MessageHandler(InMessageHandler)
{
	for ( int32 ControllerIndex=0; ControllerIndex < MAX_NUM_XINPUT_CONTROLLERS; ++ControllerIndex )
	{
		FControllerState& ControllerState = ControllerStates[ControllerIndex];
		FMemory::Memzero( &ControllerState, sizeof(FControllerState) );

		ControllerState.ControllerId = ControllerIndex;
		ControllerState.LeftTriggerReleaseDeadZone = FDynamicReleaseDeadZone();
		ControllerState.RightTriggerReleaseDeadZone = FDynamicReleaseDeadZone();
	}

	bIsGamepadAttached = false;
	bNeedsControllerStateUpdate = true;
	InitialButtonRepeatDelay = 0.2f;
	ButtonRepeatDelay = 0.1f;

	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("InitialButtonRepeatDelay"), InitialButtonRepeatDelay, GInputIni);
	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("ButtonRepeatDelay"), ButtonRepeatDelay, GInputIni);

	// In the engine, all controllers map to xbox controllers for consistency 
	X360ToXboxControllerMapping[0] = 0;		// A
	X360ToXboxControllerMapping[1] = 1;		// B
	X360ToXboxControllerMapping[2] = 2;		// X
	X360ToXboxControllerMapping[3] = 3;		// Y
	X360ToXboxControllerMapping[4] = 4;		// L1
	X360ToXboxControllerMapping[5] = 5;		// R1
	X360ToXboxControllerMapping[6] = 7;		// Back 
	X360ToXboxControllerMapping[7] = 6;		// Start
	X360ToXboxControllerMapping[8] = 8;		// Left thumbstick
	X360ToXboxControllerMapping[9] = 9;		// Right thumbstick
	X360ToXboxControllerMapping[10] = 10;	// L2
	X360ToXboxControllerMapping[11] = 11;	// R2
	X360ToXboxControllerMapping[12] = 12;	// Dpad up
	X360ToXboxControllerMapping[13] = 13;	// Dpad down
	X360ToXboxControllerMapping[14] = 14;	// Dpad left
	X360ToXboxControllerMapping[15] = 15;	// Dpad right
	X360ToXboxControllerMapping[16] = 16;	// Left stick up
	X360ToXboxControllerMapping[17] = 17;	// Left stick down
	X360ToXboxControllerMapping[18] = 18;	// Left stick left
	X360ToXboxControllerMapping[19] = 19;	// Left stick right
	X360ToXboxControllerMapping[20] = 20;	// Right stick up
	X360ToXboxControllerMapping[21] = 21;	// Right stick down
	X360ToXboxControllerMapping[22] = 22;	// Right stick left
	X360ToXboxControllerMapping[23] = 23;	// Right stick right

	Buttons[0] = FGamepadKeyNames::FaceButtonBottom;
	Buttons[1] = FGamepadKeyNames::FaceButtonRight;
	Buttons[2] = FGamepadKeyNames::FaceButtonLeft;
	Buttons[3] = FGamepadKeyNames::FaceButtonTop;
	Buttons[4] = FGamepadKeyNames::LeftShoulder;
	Buttons[5] = FGamepadKeyNames::RightShoulder;
	Buttons[6] = FGamepadKeyNames::SpecialRight;
	Buttons[7] = FGamepadKeyNames::SpecialLeft;
	Buttons[8] = FGamepadKeyNames::LeftThumb;
	Buttons[9] = FGamepadKeyNames::RightThumb;
	Buttons[10] = FGamepadKeyNames::LeftTriggerThreshold;
	Buttons[11] = FGamepadKeyNames::RightTriggerThreshold;
	Buttons[12] = FGamepadKeyNames::DPadUp;
	Buttons[13] = FGamepadKeyNames::DPadDown;
	Buttons[14] = FGamepadKeyNames::DPadLeft;
	Buttons[15] = FGamepadKeyNames::DPadRight;
	Buttons[16] = FGamepadKeyNames::LeftStickUp;
	Buttons[17] = FGamepadKeyNames::LeftStickDown;
	Buttons[18] = FGamepadKeyNames::LeftStickLeft;
	Buttons[19] = FGamepadKeyNames::LeftStickRight;
	Buttons[20] = FGamepadKeyNames::RightStickUp;
	Buttons[21] = FGamepadKeyNames::RightStickDown;
	Buttons[22] = FGamepadKeyNames::RightStickLeft;
	Buttons[23] = FGamepadKeyNames::RightStickRight;
}

float ShortToNormalizedFloat(int16 AxisVal)
{
	// normalize [-32768..32767] -> [-1..1]
	const float Norm = (AxisVal <= 0 ? 32768.f : 32767.f);
	return float(AxisVal) / Norm;
}

static FName XInputInterfaceName = FName("XInputInterface");
static FString XInputControllerIdentifier = TEXT("XInputController");

void XInputInterface::GetPlatformUserAndDevice(int32 InControllerId, EInputDeviceConnectionState InDeviceState, FPlatformUserId& OutPlatformUserId, FInputDeviceId& OutDeviceId)
{
	if (bIsPrimaryDevice)
	{
		OutDeviceId = InternalDeviceIdMappings.GetOrCreateDeviceId(InControllerId);

		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

		// If we have just been connected, then get the new platform user for a new device connection
		if (InDeviceState == EInputDeviceConnectionState::Connected)
		{
			OutPlatformUserId = DeviceMapper.GetPlatformUserForNewlyConnectedDevice();
		}
		// If we have been disconnected, remap this device to the unpaired device user
		else if (InDeviceState == EInputDeviceConnectionState::Disconnected)
		{
			OutPlatformUserId = DeviceMapper.GetUserForUnpairedInputDevices();
		}
		else
		{
			OutPlatformUserId = DeviceMapper.GetUserForInputDevice(OutDeviceId);
		}

		// If the controller is connected now but was not before, refresh the information
		if (InDeviceState == EInputDeviceConnectionState::Connected)
		{
			// Registering this device with FInputDeviceRegistry so that systems can look up metadata about this device if they
			// desire, such as icons specific to this controller type.
			FInputDeviceDescriptor Desc;
			Desc.InputDeviceName          = XInputInterfaceName;
			Desc.HardwareDeviceHandle     = OutDeviceId;
			Desc.HardwareDeviceIdentifier = FName(*XInputControllerIdentifier);
			FInputDeviceRegistry::RegisterDevice(Desc);
			
			DeviceMapper.Internal_MapInputDeviceToUser(OutDeviceId, OutPlatformUserId, InDeviceState);
		}
	}
	else
	{
		// Use the controller id as the device id for secondary input devices not connected to the input system.
		OutDeviceId = FInputDeviceId::CreateFromInternalId(InControllerId);
	}
}

void XInputInterface::SetDynamicTriggerThreshold(const int32 InControllerId, const EInputDeviceTriggerMask TriggerMask, const float Threshold)
{
	for (int32 ControllerIndex = 0; ControllerIndex < MAX_NUM_XINPUT_CONTROLLERS; ++ControllerIndex)
	{
		FControllerState& ControllerState = ControllerStates[ControllerIndex];

		if (ControllerState.ControllerId == InControllerId)
		{
			switch (TriggerMask)
			{
			case EInputDeviceTriggerMask::Left:
				ControllerState.LeftTriggerReleaseDeadZone.OverrideDeadZone(Threshold);
				break;
			case EInputDeviceTriggerMask::Right:
				ControllerState.RightTriggerReleaseDeadZone.OverrideDeadZone(Threshold);
				break;
			case EInputDeviceTriggerMask::All:
				ControllerState.LeftTriggerReleaseDeadZone.OverrideDeadZone(Threshold);
				ControllerState.RightTriggerReleaseDeadZone.OverrideDeadZone(Threshold);
				break;
			case EInputDeviceTriggerMask::None:
			default:
				break;
			}
			break;
		}
	}
}

namespace UE::XInputInterface::Private
{
EInputDeviceConnectionState GetInputDeviceConnectionState(bool bWasConnected, bool bControllerStateIsConnected)
{
	if (!bWasConnected && bControllerStateIsConnected)
	{
		return EInputDeviceConnectionState::Connected;
	}
	else if (bWasConnected && !bControllerStateIsConnected)
	{
		return EInputDeviceConnectionState::Disconnected;
	}
	return EInputDeviceConnectionState::Unknown;
}

}
void XInputInterface::SendControllerEvents()
{
	bool bWereConnected[MAX_NUM_XINPUT_CONTROLLERS];
	XINPUT_STATE XInputStates[MAX_NUM_XINPUT_CONTROLLERS];

	bIsGamepadAttached = false;
	for ( int32 ControllerIndex=0; ControllerIndex < MAX_NUM_XINPUT_CONTROLLERS; ++ControllerIndex )
	{
		FControllerState& ControllerState = ControllerStates[ControllerIndex];

		bWereConnected[ControllerIndex] = ControllerState.bIsConnected;

		if (ControllerState.bIsConnected || bNeedsControllerStateUpdate || ForceControllerStateUpdate != 0)
		{
			XINPUT_STATE& XInputState = XInputStates[ControllerIndex];
			FMemory::Memzero( &XInputState, sizeof(XINPUT_STATE) );

			ControllerState.bIsConnected = ( XInputGetState( ControllerIndex, &XInputState ) == ERROR_SUCCESS ) ? true : false;

			if (ControllerState.bIsConnected)
			{
				bIsGamepadAttached = true;
			}
		}
	}
		
	for ( int32 ControllerIndex = 0; ControllerIndex < MAX_NUM_XINPUT_CONTROLLERS; ++ControllerIndex )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Set input scope, there doesn't seem to be a reliable way to differentiate 360 vs Xbox one controllers so use generic name
		FInputDeviceScope InputScope(this, XInputInterfaceName, ControllerIndex, XInputControllerIdentifier);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		FControllerState& ControllerState = ControllerStates[ControllerIndex];

		const bool bWasConnected = bWereConnected[ControllerIndex];

		// If the controller is connected send events or if the controller was connected send a final event with default states so that 
		// the game doesn't think that controller buttons are still held down
		if( ControllerState.bIsConnected || bWasConnected )
		{
			const XINPUT_STATE& XInputState = XInputStates[ControllerIndex];

			FPlatformUserId PlatformUser = PLATFORMUSERID_NONE;
			FInputDeviceId InputDevice = INPUTDEVICEID_NONE;
			const EInputDeviceConnectionState ConnectionState = UE::XInputInterface::Private::GetInputDeviceConnectionState(bWasConnected, ControllerState.bIsConnected);
			GetPlatformUserAndDevice(ControllerState.ControllerId, ConnectionState, OUT PlatformUser, OUT InputDevice);

			// If the device has been disconnected, it needs to be remapped to the "Unpaired" input device
			// at the end of it's input processing. Use the last valid platform user to ensure they receive
			// "release" events for every active input.
			if (!PlatformUser.IsValid() && ConnectionState == EInputDeviceConnectionState::Disconnected)
			{
				PlatformUser = ControllerState.LastUsedValidPlatformUserId;
			}

			// If we never got a valid platform user for some reason, we can't do anything
			if (!PlatformUser.IsValid() && bIsPrimaryDevice)
			{
				return;
			}
			
			bool bLeftTriggerPressed = XInputState.Gamepad.bLeftTrigger > XINPUT_GAMEPAD_TRIGGER_THRESHOLD;
			bool bRightTriggerPressed = XInputState.Gamepad.bRightTrigger > XINPUT_GAMEPAD_TRIGGER_THRESHOLD;

			bLeftTriggerPressed = ControllerState.LeftTriggerReleaseDeadZone.IsPressed(XInputState.Gamepad.bLeftTrigger, bLeftTriggerPressed);
			bRightTriggerPressed = ControllerState.RightTriggerReleaseDeadZone.IsPressed(XInputState.Gamepad.bRightTrigger, bRightTriggerPressed);

			bool CurrentStates[MAX_NUM_CONTROLLER_BUTTONS] = {0};
		
			// Get the current state of all buttons
			CurrentStates[X360ToXboxControllerMapping[0]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_A);
			CurrentStates[X360ToXboxControllerMapping[1]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_B);
			CurrentStates[X360ToXboxControllerMapping[2]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_X);
			CurrentStates[X360ToXboxControllerMapping[3]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_Y);
			CurrentStates[X360ToXboxControllerMapping[4]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_LEFT_SHOULDER);
			CurrentStates[X360ToXboxControllerMapping[5]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_RIGHT_SHOULDER);
			CurrentStates[X360ToXboxControllerMapping[6]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_BACK);
			CurrentStates[X360ToXboxControllerMapping[7]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_START);
			CurrentStates[X360ToXboxControllerMapping[8]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_LEFT_THUMB);
			CurrentStates[X360ToXboxControllerMapping[9]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_RIGHT_THUMB);
			CurrentStates[X360ToXboxControllerMapping[10]] = !!(bLeftTriggerPressed);
			CurrentStates[X360ToXboxControllerMapping[11]] = !!(bRightTriggerPressed);
			CurrentStates[X360ToXboxControllerMapping[12]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_DPAD_UP);
			CurrentStates[X360ToXboxControllerMapping[13]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_DPAD_DOWN);
			CurrentStates[X360ToXboxControllerMapping[14]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_DPAD_LEFT);
			CurrentStates[X360ToXboxControllerMapping[15]] = !!(XInputState.Gamepad.wButtons & XINPUT_GAMEPAD_DPAD_RIGHT);
			CurrentStates[X360ToXboxControllerMapping[16]] = !!(XInputState.Gamepad.sThumbLY > XINPUT_GAMEPAD_LEFT_THUMB_DEADZONE);
			CurrentStates[X360ToXboxControllerMapping[17]] = !!(XInputState.Gamepad.sThumbLY < -XINPUT_GAMEPAD_LEFT_THUMB_DEADZONE);
			CurrentStates[X360ToXboxControllerMapping[18]] = !!(XInputState.Gamepad.sThumbLX < -XINPUT_GAMEPAD_LEFT_THUMB_DEADZONE);
			CurrentStates[X360ToXboxControllerMapping[19]] = !!(XInputState.Gamepad.sThumbLX > XINPUT_GAMEPAD_LEFT_THUMB_DEADZONE);
			CurrentStates[X360ToXboxControllerMapping[20]] = !!(XInputState.Gamepad.sThumbRY > XINPUT_GAMEPAD_RIGHT_THUMB_DEADZONE);
			CurrentStates[X360ToXboxControllerMapping[21]] = !!(XInputState.Gamepad.sThumbRY < -XINPUT_GAMEPAD_RIGHT_THUMB_DEADZONE);
			CurrentStates[X360ToXboxControllerMapping[22]] = !!(XInputState.Gamepad.sThumbRX < -XINPUT_GAMEPAD_RIGHT_THUMB_DEADZONE);
			CurrentStates[X360ToXboxControllerMapping[23]] = !!(XInputState.Gamepad.sThumbRX > XINPUT_GAMEPAD_RIGHT_THUMB_DEADZONE);

			// Send new analog data if it's different or outside the platform deadzone.
			auto OnControllerAnalog = [this, &PlatformUser, &InputDevice](const FName& GamePadKey, const auto NewAxisValue, const float NewAxisValueNormalized, auto& OldAxisValue, const auto DeadZone) {
				if (OldAxisValue != NewAxisValue || FMath::Abs((int32)NewAxisValue) > DeadZone)
				{
					MessageHandler->OnControllerAnalog(GamePadKey, PlatformUser, InputDevice, NewAxisValueNormalized);

					UE_LOGF(LogXInput, VeryVerbose, "[%s] PlatUser: %d DeviceId %d Key: '%ls'     Value: %.3f",
						__func__, PlatformUser.GetInternalId(), InputDevice.GetId(), *GamePadKey.ToString(), NewAxisValueNormalized);
				}
				OldAxisValue = NewAxisValue;
			};

			const auto& Gamepad = XInputState.Gamepad;

			OnControllerAnalog(FGamepadKeyNames::LeftAnalogX, Gamepad.sThumbLX, ShortToNormalizedFloat(Gamepad.sThumbLX), ControllerState.LeftXAnalog, XINPUT_GAMEPAD_LEFT_THUMB_DEADZONE);
			OnControllerAnalog(FGamepadKeyNames::LeftAnalogY, Gamepad.sThumbLY, ShortToNormalizedFloat(Gamepad.sThumbLY), ControllerState.LeftYAnalog, XINPUT_GAMEPAD_LEFT_THUMB_DEADZONE);

			OnControllerAnalog(FGamepadKeyNames::RightAnalogX, Gamepad.sThumbRX, ShortToNormalizedFloat(Gamepad.sThumbRX), ControllerState.RightXAnalog, XINPUT_GAMEPAD_RIGHT_THUMB_DEADZONE);
			OnControllerAnalog(FGamepadKeyNames::RightAnalogY, Gamepad.sThumbRY, ShortToNormalizedFloat(Gamepad.sThumbRY), ControllerState.RightYAnalog, XINPUT_GAMEPAD_RIGHT_THUMB_DEADZONE);

			OnControllerAnalog(FGamepadKeyNames::LeftTriggerAnalog, Gamepad.bLeftTrigger, Gamepad.bLeftTrigger / 255.f, ControllerState.LeftTriggerAnalog, XINPUT_GAMEPAD_TRIGGER_THRESHOLD);
			OnControllerAnalog(FGamepadKeyNames::RightTriggerAnalog, Gamepad.bRightTrigger, Gamepad.bRightTrigger / 255.f, ControllerState.RightTriggerAnalog, XINPUT_GAMEPAD_TRIGGER_THRESHOLD);

			const double CurrentTime = FPlatformTime::Seconds();

			// For each button check against the previous state and send the correct message if any
			for (int32 ButtonIndex = 0; ButtonIndex < MAX_NUM_CONTROLLER_BUTTONS; ++ButtonIndex)
			{
				if (CurrentStates[ButtonIndex] != ControllerState.ButtonStates[ButtonIndex])
				{
					if( CurrentStates[ButtonIndex] )
					{
						MessageHandler->OnControllerButtonPressed( Buttons[ButtonIndex], PlatformUser, InputDevice, false );
						
						UE_LOGF(LogXInput, Verbose, "[%s] OnControllerButtonPressed PlatUser: %d DeviceId %d Key: '%ls' bIsRepeat: false",
							__func__, PlatformUser.GetInternalId(), InputDevice.GetId(), *Buttons[ButtonIndex].ToString());
					}
					else
					{
						MessageHandler->OnControllerButtonReleased( Buttons[ButtonIndex], PlatformUser, InputDevice, false );
						
						UE_LOGF(LogXInput, Verbose, "[%s] OnControllerButtonReleased PlatUser: %d DeviceId %d Key: '%ls' bIsRepeat: false",
							__func__, PlatformUser.GetInternalId(), InputDevice.GetId(), *Buttons[ButtonIndex].ToString());
					}

					if ( CurrentStates[ButtonIndex] != 0 )
					{
						// this button was pressed - set the button's NextRepeatTime to the InitialButtonRepeatDelay
						ControllerState.NextRepeatTime[ButtonIndex] = CurrentTime + InitialButtonRepeatDelay;
					}
				}
				else if ( CurrentStates[ButtonIndex] != 0 && ControllerState.NextRepeatTime[ButtonIndex] <= CurrentTime )
				{
					MessageHandler->OnControllerButtonPressed( Buttons[ButtonIndex], PlatformUser, InputDevice, true );
					
					UE_LOGF(LogXInput, Verbose, "[%s] OnControllerButtonPressed PlatUser: %d DeviceId %d Key: '%ls' bIsRepeat: true",
						__func__, PlatformUser.GetInternalId(), InputDevice.GetId(), *Buttons[ButtonIndex].ToString());

					// set the button's NextRepeatTime to the ButtonRepeatDelay
					ControllerState.NextRepeatTime[ButtonIndex] = CurrentTime + ButtonRepeatDelay;
				}

				// Update the state for next time
				ControllerState.ButtonStates[ButtonIndex] = CurrentStates[ButtonIndex];
			}	

			// apply force feedback

			const float LargeValue = (ControllerState.ForceFeedback.LeftLarge > ControllerState.ForceFeedback.RightLarge ? ControllerState.ForceFeedback.LeftLarge : ControllerState.ForceFeedback.RightLarge);
			const float SmallValue = (ControllerState.ForceFeedback.LeftSmall > ControllerState.ForceFeedback.RightSmall ? ControllerState.ForceFeedback.LeftSmall : ControllerState.ForceFeedback.RightSmall);

			if (!FMath::IsNearlyEqual(LargeValue, ControllerState.LastLargeValue) || !FMath::IsNearlyEqual(SmallValue, ControllerState.LastSmallValue))
			{
				XINPUT_VIBRATION VibrationState;
				VibrationState.wLeftMotorSpeed = ( ::WORD ) ( LargeValue * 65535.0f );
				VibrationState.wRightMotorSpeed = ( ::WORD ) ( SmallValue * 65535.0f );
 
				XInputSetState( ( ::DWORD ) ControllerState.ControllerId, &VibrationState );

				ControllerState.LastLargeValue = LargeValue;
				ControllerState.LastSmallValue = SmallValue;
			}

			// Keep track of the last valid platform user id
			if (PlatformUser.IsValid())
			{
				ControllerState.LastUsedValidPlatformUserId = PlatformUser;	
			}

			// Remap the input device to the now invalid platform user at the end of the frame
			// This way we can ensure that slate gets reported the "0.0" analog values to stop any
			// active inputs upon disconnect
			if (ConnectionState == EInputDeviceConnectionState::Disconnected)
			{
				IPlatformInputDeviceMapper::Get().Internal_MapInputDeviceToUser(InputDevice, IPlatformInputDeviceMapper::Get().GetUserForUnpairedInputDevices(), ConnectionState);
			}
		}
	}

	bNeedsControllerStateUpdate = false;
}


void XInputInterface::SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	MessageHandler = InMessageHandler;
}

void XInputInterface::SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property)
{
	// This should only get called from WindowsApplication.cpp when Windows detects a device change.
	static const FName UpdateRequestedName = TEXT("Request_Device_Update");
	if (Property)
	{
		if (Property->Name == UpdateRequestedName)
		{
			SetNeedsControllerStateUpdate();
		}
		else if (Property->Name == FInputDeviceTriggerDynamicReleaseDeadZoneProperty::PropertyName())
		{
			const FInputDeviceTriggerDynamicReleaseDeadZoneProperty* TriggerReleaseThreshold = static_cast<const FInputDeviceTriggerDynamicReleaseDeadZoneProperty*>(Property);
			SetDynamicTriggerThreshold(ControllerId, TriggerReleaseThreshold->AffectedTriggers, TriggerReleaseThreshold->DeadZone);
		}
	}
}

void XInputInterface::SetChannelValue( int32 ControllerId, const FForceFeedbackChannelType ChannelType, const float Value )
{
	const FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetPlatformUserForUserIndex(ControllerId);

	UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get();
	check(DeviceSubsystem);
	
	// Get the latest gamepad input device
	const FInputDeviceId MostRecentDevice = DeviceSubsystem->GetLatestDeviceOfType(UserId, EHardwareDevicePrimaryType::Gamepad);
	
	if (ControllerId >= 0 && ControllerId < MAX_NUM_XINPUT_CONTROLLERS)
	{
		for (int32 i = 0; i < MAX_NUM_XINPUT_CONTROLLERS; ++i)
		{
			FControllerState& ControllerState = ControllerStates[ i ];

			const FInputDeviceId CurrentDeviceId = InternalDeviceIdMappings.FindDeviceId(ControllerState.ControllerId);

			if (ControllerState.bIsConnected &&
				ControllerState.LastUsedValidPlatformUserId == UserId &&
				CurrentDeviceId.IsValid() && MostRecentDevice == CurrentDeviceId)
			{
				switch( ChannelType )
				{
				case FForceFeedbackChannelType::LEFT_LARGE:
					ControllerState.ForceFeedback.LeftLarge = Value;
					break;

				case FForceFeedbackChannelType::LEFT_SMALL:
					ControllerState.ForceFeedback.LeftSmall = Value;
					break;

				case FForceFeedbackChannelType::RIGHT_LARGE:
					ControllerState.ForceFeedback.RightLarge = Value;
					break;

				case FForceFeedbackChannelType::RIGHT_SMALL:
					ControllerState.ForceFeedback.RightSmall = Value;
					break;
				}
			}
		}	
	}
}

void XInputInterface::SetChannelValues( int32 ControllerId, const FForceFeedbackValues &Values )
{
	const FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetPlatformUserForUserIndex(ControllerId);

	UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get();
	check(DeviceSubsystem);
	const FInputDeviceId MostRecentDevice = DeviceSubsystem->GetLatestDeviceOfType(UserId, EHardwareDevicePrimaryType::Gamepad);
	
	if (ControllerId >= 0 && ControllerId < MAX_NUM_XINPUT_CONTROLLERS)
	{
		for (int32 i = 0; i < MAX_NUM_XINPUT_CONTROLLERS; ++i)
		{
			FControllerState& ControllerState = ControllerStates[ i ];
			
			const FInputDeviceId CurrentDeviceId = InternalDeviceIdMappings.FindDeviceId(ControllerState.ControllerId);

			if (ControllerState.bIsConnected &&
				ControllerState.LastUsedValidPlatformUserId == UserId)
			{
				if (CurrentDeviceId.IsValid() && CurrentDeviceId == MostRecentDevice)
				{
					ControllerState.ForceFeedback = Values;
				}
				// Ensure that other gamepads mapped to this user are zero'd out if they are not currently active for force feedback
				// This way you don't get a controller which is stuck in a force feedback loop if you change devices mid-effect if there are
				// multiple devices mapped to the same user
				else
				{
					ControllerState.ForceFeedback = FForceFeedbackValues();
				}
			}			
		}		
	}
}
