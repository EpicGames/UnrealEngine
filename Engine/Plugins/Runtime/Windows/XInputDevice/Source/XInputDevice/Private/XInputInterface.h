// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/IInputInterface.h"
#include "IInputDevice.h"
#include "GenericPlatform/GamepadUtils.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericInputDeviceMap.h"
#include "Misc/CoreMiscDefines.h"

/** Max number of controllers. */
#define MAX_NUM_XINPUT_CONTROLLERS 4

/** Max number of controller buttons.  Must be < 256*/
#define MAX_NUM_CONTROLLER_BUTTONS 24

enum class FForceFeedbackChannelType;

/**
 * Interface class for XInput devices (xbox 360 controller)                 
 */
class XInputInterface : public IInputDevice
{
public:

	static TSharedRef< XInputInterface > Create( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler, bool bShouldBePrimaryDevice );

	/**
	 * Poll for controller state and send events if needed
	 *
	 * @param PathToJoystickCaptureWidget	The path to the joystick capture widget.  If invalid this function does not poll 
	 */
	virtual void SendControllerEvents() override;

	virtual void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler ) override;

	void SetNeedsControllerStateUpdate() { bNeedsControllerStateUpdate = true; }

	/**
	* Sets the strength/speed of the given channel for the given controller id.
	* NOTE: If the channel is not supported, the call will silently fail
	*
	* @param ControllerId the id of the controller whose value is to be set
	* @param ChannelType the type of channel whose value should be set
	* @param Value strength or speed of feedback, 0.0f to 1.0f. 0.0f will disable
	*/
	virtual void SetChannelValue( int32 ControllerId, const FForceFeedbackChannelType ChannelType, const float Value ) override;

	/**
	* Sets a property for a given controller id.
	* Will be ignored for devices which don't support the property.
	*
	* @param ControllerId the id of the controller whose property is to be applied
	* @param Property Base class pointer to property that will be applied
	*/
	virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) override;

	/**
	* Sets the strength/speed of all the channels for the given controller id.
	* NOTE: Unsupported channels are silently ignored
	*
	* @param ControllerId the id of the controller whose value is to be set
	* @param Values strength or speed of feedback for all channels
	*/
	virtual void SetChannelValues( int32 ControllerId, const FForceFeedbackValues& Values ) override;

	virtual bool IsGamepadAttached() const override { return bIsGamepadAttached; }
	virtual void Tick( float DeltaTime ) override {};
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) { return false; }

	// XInputGetState() is a stateless poll with no game-thread restrictions per Microsoft docs.
	virtual EInputDeviceThreadAffinity GetThreadAffinity() const override { return EInputDeviceThreadAffinity::InputThreadSafe; }

private:

	XInputInterface( const TSharedRef< FGenericApplicationMessageHandler >& MessageHandler, bool bShouldBePrimaryDevice );

	/**
	 * Maps the given controller id to the platform user and device id. It will use the device mapper when used with the input system.
	 */
	void GetPlatformUserAndDevice( int32 InControllerId, EInputDeviceConnectionState InDeviceState, FPlatformUserId& OutPlatformUserId, FInputDeviceId& OutDeviceId );

	/** Sets dynamic trigger release threshold for the given trigger(s) on the given controller. */
	void SetDynamicTriggerThreshold(const int32 ControllerId, const EInputDeviceTriggerMask TriggerMask, const float Threshold);

	struct FControllerState
	{
		/** Last frame's button states, so we only send events on edges */
		bool ButtonStates[MAX_NUM_CONTROLLER_BUTTONS];

		/** Next time a repeat event should be generated for each button */
		double NextRepeatTime[MAX_NUM_CONTROLLER_BUTTONS];

		/** Raw Left thumb x analog value */
		int16 LeftXAnalog;

		/** Raw left thumb y analog value */
		int16 LeftYAnalog;

		/** Raw Right thumb x analog value */
		int16 RightXAnalog;

		/** Raw Right thumb x analog value */
		int16 RightYAnalog;

		/** Left Trigger analog value */
		uint8 LeftTriggerAnalog;

		/** Right trigger analog value */
		uint8 RightTriggerAnalog;

		/** If the controller is currently connected */
		bool bIsConnected;

		/** Id of the controller */
		int32 ControllerId;
	
		/** Current force feedback values */
		FForceFeedbackValues ForceFeedback;

		float LastLargeValue;
		float LastSmallValue;

		/**
		 * The last valid FPlatformUserId that can be used on the frame where the input device is disconnected
		 */
		FPlatformUserId LastUsedValidPlatformUserId;

		/** Dynamic Release DeadZone for Left Trigger */
		FDynamicReleaseDeadZone LeftTriggerReleaseDeadZone;

		/** Dynamic Release DeadZone for Right Trigger */
		FDynamicReleaseDeadZone RightTriggerReleaseDeadZone;
	};

	/** If we've been notified by the system that the controller state may have changed */
	bool bNeedsControllerStateUpdate;

	bool bIsGamepadAttached;

	/** Indicates if this device is operating as a primary device and thus part of game input system. */
	bool bIsPrimaryDevice;

	/** In the engine, all controllers map to xbox controllers for consistency */
	uint8	X360ToXboxControllerMapping[MAX_NUM_CONTROLLER_BUTTONS];

	/** Controller states */
	FControllerState ControllerStates[MAX_NUM_XINPUT_CONTROLLERS];

	/** Delay before sending a repeat message after a button was first pressed */
	float InitialButtonRepeatDelay;

	/** Delay before sending a repeat message after a button has been pressed for a while */
	float ButtonRepeatDelay;

	/** Array of gamepad button names where the index in the array is the XInput button */
	FGamepadKeyNames::Type Buttons[MAX_NUM_CONTROLLER_BUTTONS];

	/** Reference to the message handler, used to send the input state to the application. */
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;

	/**
	 * A map of XInput device identifiers (int32's) to their uniquely assigned
	 * FInputDeviceId from the engine.
	 */
	TInputDeviceMap<int32> InternalDeviceIdMappings;
};
