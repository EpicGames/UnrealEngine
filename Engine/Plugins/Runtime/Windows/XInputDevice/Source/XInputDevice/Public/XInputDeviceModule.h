// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDeviceModule.h"

/**
 * Input Device Module for XInput, an input API for Windows.
 */
class FXInputDeviceModule final : public IInputDeviceModule
{
public:
	//~ Begin IInputDeviceModule interface
	virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, FInputDeviceCreationParameters InParameters) override;
	virtual const TCHAR* GetPreferredDeviceAPIString() const override;
	//~ End IInputDeviceModule interface
};
