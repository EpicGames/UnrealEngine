// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool.Rules
{
	public class XInputDevice : ModuleRules
	{
		public XInputDevice(ReadOnlyTargetRules Target) : base(Target)
		{
			// Uncomment this line to make for easier debugging
			//OptimizeCode = CodeOptimization.Never;
						
			// Enable cast warnings as errors
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"ApplicationCore",
					"Engine",
					"InputDevice",
				}
			);

			// XInput is only supported on Windows
			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target,
					"XInput"
				);
			}
			else
			{
				Console.Error.Write("XInput is only supported on the Windows target platform group!");
			}
		}
	}
}