{
	"FileVersion": 3,
	"FriendlyName": "XInput Device",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "XInput is a Game Controller API for Windows.",
	"Category": "Input Devices",
	"EnabledByDefault": true,
	"IsBetaVersion": false,
	"CanContainContent": false,
    "SupportedPrograms": ["LiveLinkHub"],
	"SupportedTargetPlatforms":
	[
		"Win64"
	],
	"Modules":
	[
		{
			"Name": "XInputDevice",
			"Type": "Runtime",
            "ProgramAllowList": ["LiveLinkHub"],
			"PlatformAllowList":
			[
				"Win64"
			]
		}
	]
}
